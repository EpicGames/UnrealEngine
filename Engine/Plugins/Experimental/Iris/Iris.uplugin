{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Iris",
	"Description": "Iris networking.",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "Iris",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
