// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Iris : ModuleRules
{
    public Iris(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
            new string[]
			{
                "Core",
			}
		);

        SetupIrisSupport(Target);
	}
}
