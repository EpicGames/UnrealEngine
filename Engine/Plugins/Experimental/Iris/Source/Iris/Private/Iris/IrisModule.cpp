// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FIrisModule : public IModuleInterface
{
private:
	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModule("IrisCore", ELoadModuleFlags::None);
	}
};

IMPLEMENT_MODULE(FIrisModule, Iris);
